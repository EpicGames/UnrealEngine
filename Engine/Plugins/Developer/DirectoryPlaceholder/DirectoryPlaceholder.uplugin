{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "1.0",
	"FriendlyName": "Directory Placeholder",
	"Description": "Adds a lightweight Directory Placeholder asset type, which can be added to otherwise empty folders in order to add them to source control.",
	"Category": "Source Control",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"DocsURL": "",
	"MarketplaceURL": "",
	"SupportURL": "",
	"CanContainContent": false,
	"IsBetaVersion": false,
	"IsExperimentalVersion": true,
	"Hidden": false,
	"Installed": false,
	"Modules": [
		{
			"Name": "DirectoryPlaceholder",
			"Type": "Editor",
			"LoadingPhase": "Default"
		}
	]
}