// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class DirectoryPlaceholder : ModuleRules
	{
		public DirectoryPlaceholder(ReadOnlyTargetRules Target) : base(Target)
		{
			PublicDependencyModuleNames.AddRange(
				new string[]
				{
				}
			);

			PrivateDependencyModuleNames.AddRange(
				new string[]
				{
					"AssetDefinition",
					"ContentBrowser",
					"ContentBrowserData",
					"Core",
					"CoreUObject",
					"DeveloperSettings",
					"Engine",
					"Projects",
					"Slate",
					"SlateCore",
					"ToolMenus",
					"UnrealEd",
				}
			);
		}
	}
}
