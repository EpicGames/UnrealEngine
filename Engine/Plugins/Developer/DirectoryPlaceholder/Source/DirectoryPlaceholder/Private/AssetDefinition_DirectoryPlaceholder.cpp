// Copyright Epic Games, Inc. All Rights Reserved.

#include "AssetDefinition_DirectoryPlaceholder.h"
#include "DirectoryPlaceholder.h"

#define LOCTEXT_NAMESPACE "AssetTypeActions"

FText UAssetDefinition_DirectoryPlaceholder::GetAssetDisplayName() const
{
	return LOCTEXT("AssetTypeActions_DirectoryPlaceholder", "Directory Placeholder");
}

FLinearColor UAssetDefinition_DirectoryPlaceholder::GetAssetColor() const
{
	return FLinearColor::Gray;
}

TSoftClassPtr<> UAssetDefinition_DirectoryPlaceholder::GetAssetClass() const
{
	return UDirectoryPlaceholder::StaticClass();
}

TConstArrayView<FAssetCategoryPath> UAssetDefinition_DirectoryPlaceholder::GetAssetCategories() const
{
	static const FAssetCategoryPath Categories[] = { FAssetCategoryPath(EAssetCategoryPaths::Misc, LOCTEXT("DirectoryPlaceholder_CategorySection", "Other"), ECategoryMenuType::Section) };
	return Categories;
}

EAssetCommandResult UAssetDefinition_DirectoryPlaceholder::OpenAssets(const FAssetOpenArgs& OpenArgs) const
{
	return EAssetCommandResult::Unhandled;
}

#undef LOCTEXT_NAMESPACE
