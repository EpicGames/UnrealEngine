// Copyright Epic Games, Inc. All Rights Reserved.

#include "DirectoryPlaceholder.h"
#include "FrontendFilterBase.h"

#define LOCTEXT_NAMESPACE "DirectoryPlaceholder"

class FFrontendFilter_DirectoryPlaceholder : public FFrontendFilter
{
public:
	FFrontendFilter_DirectoryPlaceholder(TSharedPtr<FFrontendFilterCategory> InCategory)
		: FFrontendFilter(InCategory)
	{
	}

	// Begin FFilterBase interface
	virtual FString GetName() const override { return TEXT("DirectoryPlaceholderFilter"); }
	virtual FText GetDisplayName() const override { return LOCTEXT("DirectoryPlaceholderFilterName", "Show Directory Placeholders"); }
	virtual FText GetToolTipText() const override { return LOCTEXT("DirectoryPlaceholderFilterTooltip", "Show Directory Placeholders"); }

	/** This is an inverse filter to prevent the asset view from recursively displaying all assets */
	virtual bool IsInverseFilter() const override { return true; } 
	// End FFilterBase interface

	// Begin IFilter interface
	virtual bool PassesFilter(FAssetFilterType InItem) const override;
	// End IFilter interface
};

bool FFrontendFilter_DirectoryPlaceholder::PassesFilter(FAssetFilterType InItem) const
{
	const FContentBrowserItemDataAttributeValue ClassValue = InItem.GetItemAttribute(NAME_Class);
	return !ClassValue.IsValid() || ClassValue.GetValue<FString>() != UDirectoryPlaceholder::StaticClass()->GetPathName();
}

void UDirectoryPlaceholderSearchFilter::AddFrontEndFilterExtensions(TSharedPtr<FFrontendFilterCategory> DefaultCategory, TArray<TSharedRef<class FFrontendFilter>>& InOutFilterList) const
{
	InOutFilterList.Add(MakeShareable(new FFrontendFilter_DirectoryPlaceholder(DefaultCategory)));
}

#undef LOCTEXT_NAMESPACE
