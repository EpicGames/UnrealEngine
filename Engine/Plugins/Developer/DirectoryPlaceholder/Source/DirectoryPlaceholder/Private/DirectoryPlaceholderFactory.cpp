// Copyright Epic Games, Inc. All Rights Reserved.

#include "DirectoryPlaceholderFactory.h"
#include "DirectoryPlaceholder.h"

#define LOCTEXT_NAMESPACE "DirectoryPlaceholderFactory"

UDirectoryPlaceholderFactory::UDirectoryPlaceholderFactory()
{
	SupportedClass = UDirectoryPlaceholder::StaticClass();
	bCreateNew = true;
	bEditAfterNew = true;
}

bool UDirectoryPlaceholderFactory::CanCreateNew() const
{
	return true;
}

UObject* UDirectoryPlaceholderFactory::FactoryCreateNew(UClass* Class, UObject* InParent, FName Name, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn)
{
	check(Class->IsChildOf(UDirectoryPlaceholder::StaticClass()));
	return NewObject<UDirectoryPlaceholder>(InParent, Class, Name, Flags);
}

#undef LOCTEXT_NAMESPACE
