// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Factories/Factory.h"

#include "DirectoryPlaceholderFactory.generated.h"

/** Implements a factory for UDirectoryPlaceholder objects */
UCLASS(hidecategories=Object)
class UDirectoryPlaceholderFactory : public UFactory
{
	GENERATED_BODY()

protected:
	UDirectoryPlaceholderFactory();

	// Begin UFactory Interface
	virtual UObject* FactoryCreateNew(UClass* Class, UObject* InParent, FName Name, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn) override;
	virtual bool CanCreateNew() const override;
	// End UFactory Interface
};
