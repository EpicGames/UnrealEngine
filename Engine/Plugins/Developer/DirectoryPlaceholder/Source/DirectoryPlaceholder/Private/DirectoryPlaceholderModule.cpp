// Copyright Epic Games, Inc. All Rights Reserved.

#include "Modules/ModuleManager.h"

#include "Algo/AnyOf.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "AssetToolsModule.h"
#include "ContentBrowserItemPath.h"
#include "ContentBrowserMenuContexts.h"
#include "ContentBrowserModule.h"
#include "DirectoryPlaceholder.h"
#include "DirectoryPlaceholderFactory.h"
#include "DirectoryPlaceholderSettings.h"
#include "DirectoryPlaceholderUtils.h"
#include "DirectoryWatcherModule.h"
#include "FileHelpers.h"
#include "IDirectoryWatcher.h"
#include "Interfaces/IProjectManager.h"
#include "ObjectTools.h"
#include "ToolMenus.h"

DEFINE_LOG_CATEGORY_STATIC(LogDirectoryPlaceholder, Log, All)

#define LOCTEXT_NAMESPACE "DirectoryPlaceholderModule"

/** 
 * Directory Placeholder Module
 * Manages automatic creation and deletion of placeholder assets, allowing folders to be checked in to source control.
 */
class FDirectoryPlaceholderModule : public IModuleInterface
{
public:

	//~ Begin IModuleInterface interface
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;
	//~ End IModuleInterface interface

private:
	/** Register/Unregister all enabled Directory Watchers */
	void RegisterDirectoryWatchers();
	void UnregisterDirectoryWatchers();

	/** Register/Unregister a Directory Watcher for the input path */
	void RegisterDirectoryWatcher(const FString& Path, FDelegateHandle& Handle);
	void UnregisterDirectoryWatcher(const FString& Path, FDelegateHandle& Handle);

	/** Register/Unregister a Directory Watcher for each additional (external) plugin path */
	void RegisterDirectoryWatcher_AdditionalPlugins();
	void UnregisterDirectoryWatcher_AdditionalPlugins();

	/** Callback when the directory watcher detects a file/folder change */
	void OnDirectoryChanged(const TArray<FFileChangeData>& InFileChanges);

	/** Executes when one or more folders are being deleted in the content browser */
	void OnDeleteFolders(const TArray<FContentBrowserItemPath>& PathsToDelete);

#if WITH_EDITOR
	/** Callback when one of the directory placeholder settings has changed */
	void OnDirectoryPlaceholderSettingsChanged(UObject* Object, FPropertyChangedEvent& PropertyChangedEvent);
#endif // WITH_EDITOR

private:
	/** Handles to each of the registerd directory watcher delegates, used to unregister them later */
	FDelegateHandle ProjectContentHandle;
	FDelegateHandle ProjectPluginsHandle;
	TMap<FString, FDelegateHandle> AdditionalPluginHandles;

	FDelegateHandle OnSettingsChangedHandle;
	FDelegateHandle OnDeleteFoldersHandle;
};

void FDirectoryPlaceholderModule::StartupModule()
{
	// Only enable this behavior in an interactive editor
	if (GIsEditor && !IsRunningCommandlet())
	{
		// Delay registration of the directory watchers until after the asset registry has finished its initial scan
		const FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>("AssetRegistry");
		AssetRegistryModule.Get().OnFilesLoaded().AddRaw(this, &FDirectoryPlaceholderModule::RegisterDirectoryWatchers);

		// Register a callback to execute when one or more folders are being deleted in the content browser
		FContentBrowserModule& ContentBrowserModule = FModuleManager::Get().GetModuleChecked<FContentBrowserModule>(TEXT("ContentBrowser"));
		OnDeleteFoldersHandle = ContentBrowserModule.GetOnDeleteFolders().AddRaw(this, &FDirectoryPlaceholderModule::OnDeleteFolders);

		// Register a callback to execute when one of the directory placeholders settings has changed
#if WITH_EDITOR
		UDirectoryPlaceholderSettings* Settings = GetMutableDefault<UDirectoryPlaceholderSettings>();
		OnSettingsChangedHandle = Settings->OnSettingChanged().AddRaw(this, &FDirectoryPlaceholderModule::OnDirectoryPlaceholderSettingsChanged);
#endif // WITH_EDITOR

		// Extend the content browser folder context menu with the option to cleanup directory placeholders
		if (UToolMenu* Menu = UToolMenus::Get()->ExtendMenu("ContentBrowser.FolderContextMenu"))
		{
			FToolMenuSection& BulkOpsSection = Menu->FindOrAddSection("PathContextBulkOperations");

			FToolMenuEntry& Entry = BulkOpsSection.AddDynamicEntry("CleanupDirectoryPlaceholders", FNewToolMenuSectionDelegate::CreateLambda([this](FToolMenuSection& InSection)
				{
					UContentBrowserFolderContext* Context = InSection.FindContext<UContentBrowserFolderContext>();
					if (Context && Context->NumAssetPaths > 0)
					{
						const TArray<FString>& Paths = Context->GetSelectedPackagePaths();

						FToolMenuEntry& Entry = InSection.AddMenuEntry(
							"CleanupDirectoryPlaceholders",
							LOCTEXT("CleanupDirectoryPlaceholdersLabel", "Cleanup Directory Placeholders"),
							LOCTEXT("CleanupDirectoryPlaceholdersToolTip", "Delete all unnecessary placeholder assets in this folder (and sub-folders)"),
							FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.FolderClosed"),
							FUIAction(FExecuteAction::CreateLambda([Paths]() { UDirectoryPlaceholderLibrary::CleanupPlaceholdersInPaths(Paths); }))
						);

						Entry.InsertPosition = FToolMenuInsert("Delete", EToolMenuInsertType::After);
					}
				}));
		}
	}
}

void FDirectoryPlaceholderModule::ShutdownModule()
{
	if (GIsEditor && !IsRunningCommandlet())
	{
		UnregisterDirectoryWatchers();

		FContentBrowserModule& ContentBrowserModule = FModuleManager::Get().GetModuleChecked<FContentBrowserModule>(TEXT("ContentBrowser"));
		ContentBrowserModule.GetOnDeleteFolders().Remove(OnDeleteFoldersHandle);

#if WITH_EDITOR
		if (UObjectInitialized())
		{
			GetMutableDefault<UDirectoryPlaceholderSettings>()->OnSettingChanged().RemoveAll(this);
		}
#endif
	}
}

void FDirectoryPlaceholderModule::RegisterDirectoryWatcher(const FString& Path, FDelegateHandle& Handle)
{
	// Register a callback with the directory watcher module to be notified about file/folder changes matching the input path
	FDirectoryWatcherModule& DirectoryWatcherModule = FModuleManager::GetModuleChecked<FDirectoryWatcherModule>("DirectoryWatcher");
	if (IDirectoryWatcher* DirectoryWatcher = DirectoryWatcherModule.Get())
	{
		if (FPaths::DirectoryExists(Path) && !Handle.IsValid())
		{
			DirectoryWatcher->RegisterDirectoryChangedCallback_Handle(
				Path,
				IDirectoryWatcher::FDirectoryChanged::CreateRaw(this, &FDirectoryPlaceholderModule::OnDirectoryChanged),
				Handle,
				IDirectoryWatcher::WatchOptions::IncludeDirectoryChanges
			);
		}
	}
}

void FDirectoryPlaceholderModule::UnregisterDirectoryWatcher(const FString& Path, FDelegateHandle& Handle)
{
	FDirectoryWatcherModule& DirectoryWatcherModule = FModuleManager::GetModuleChecked<FDirectoryWatcherModule>("DirectoryWatcher");
	if (IDirectoryWatcher* DirectoryWatcher = DirectoryWatcherModule.Get())
	{
		if (Handle.IsValid())
		{
			DirectoryWatcher->UnregisterDirectoryChangedCallback_Handle(Path, Handle);
			Handle.Reset();
		}
	}
}

void FDirectoryPlaceholderModule::RegisterDirectoryWatchers()
{
	const UDirectoryPlaceholderSettings* Settings = GetDefault<UDirectoryPlaceholderSettings>();
	if (Settings->bAutomaticallyCreatePlaceholders)
	{
		if (Settings->bAutomaticallyCreatePlaceholdersInProjectContent)
		{
			RegisterDirectoryWatcher(FPaths::ProjectContentDir(), ProjectContentHandle);
		}
		if (Settings->bAutomaticallyCreatePlaceholdersInProjectPlugins)
		{
			RegisterDirectoryWatcher(FPaths::ProjectPluginsDir(), ProjectPluginsHandle);
		}
		if (Settings->bAutomaticallyCreatePlaceholdersInAdditionalPlugins)
		{
			RegisterDirectoryWatcher_AdditionalPlugins();
		}
	}
}

void FDirectoryPlaceholderModule::UnregisterDirectoryWatchers()
{
	UnregisterDirectoryWatcher(FPaths::ProjectContentDir(), ProjectContentHandle);
	UnregisterDirectoryWatcher(FPaths::ProjectPluginsDir(), ProjectPluginsHandle);
	UnregisterDirectoryWatcher_AdditionalPlugins();
}

void FDirectoryPlaceholderModule::RegisterDirectoryWatcher_AdditionalPlugins()
{
	for (const FString& AdditionalPluginDirectory : IProjectManager::Get().GetAdditionalPluginDirectories())
	{
		FDelegateHandle& AdditionalPluginHandle = AdditionalPluginHandles.Add(AdditionalPluginDirectory);
		RegisterDirectoryWatcher(AdditionalPluginDirectory, AdditionalPluginHandle);
	}
}

void FDirectoryPlaceholderModule::UnregisterDirectoryWatcher_AdditionalPlugins()
{
	for (TPair<FString, FDelegateHandle>& AdditionalPluginHandle : AdditionalPluginHandles)
	{
		UnregisterDirectoryWatcher(AdditionalPluginHandle.Key, AdditionalPluginHandle.Value);
	}
	AdditionalPluginHandles.Empty();
}

#if WITH_EDITOR
void FDirectoryPlaceholderModule::OnDirectoryPlaceholderSettingsChanged(UObject* Object, FPropertyChangedEvent& PropertyChangedEvent)
{
	const FName PropertyName = PropertyChangedEvent.GetPropertyName();

	const UDirectoryPlaceholderSettings* Settings = GetDefault<UDirectoryPlaceholderSettings>();

	if (PropertyName == GET_MEMBER_NAME_CHECKED(UDirectoryPlaceholderSettings, bAutomaticallyCreatePlaceholders))
	{
		Settings->bAutomaticallyCreatePlaceholders ? RegisterDirectoryWatchers() : UnregisterDirectoryWatchers();
	}
	else if (PropertyName == GET_MEMBER_NAME_CHECKED(UDirectoryPlaceholderSettings, bAutomaticallyCreatePlaceholdersInProjectContent))
	{
		Settings->bAutomaticallyCreatePlaceholdersInProjectContent ? RegisterDirectoryWatcher(FPaths::ProjectContentDir(), ProjectContentHandle) : UnregisterDirectoryWatcher(FPaths::ProjectContentDir(), ProjectContentHandle);
	}
	else if (PropertyName == GET_MEMBER_NAME_CHECKED(UDirectoryPlaceholderSettings, bAutomaticallyCreatePlaceholdersInProjectPlugins))
	{
		Settings->bAutomaticallyCreatePlaceholdersInProjectPlugins ? RegisterDirectoryWatcher(FPaths::ProjectPluginsDir(), ProjectPluginsHandle) : UnregisterDirectoryWatcher(FPaths::ProjectPluginsDir(), ProjectPluginsHandle);
	}
	else if (PropertyName == GET_MEMBER_NAME_CHECKED(UDirectoryPlaceholderSettings, bAutomaticallyCreatePlaceholdersInAdditionalPlugins))
	{
		Settings->bAutomaticallyCreatePlaceholdersInAdditionalPlugins ? RegisterDirectoryWatcher_AdditionalPlugins() : UnregisterDirectoryWatcher_AdditionalPlugins();
	}
}
#endif //WITH_EDITOR

void FDirectoryPlaceholderModule::OnDirectoryChanged(const TArray<FFileChangeData>& InFileChanges)
{
	TArray<UPackage*> PackagesToSave;

	for (const FFileChangeData& FileChange : InFileChanges)
	{
		// Check if the FileChangeData represent a directory that was just added
		if (FPaths::DirectoryExists(FileChange.Filename) && (FileChange.Action == FFileChangeData::FCA_Added))
		{
			FString PackagePath;
			if (!FPackageName::TryConvertFilenameToLongPackageName(FileChange.Filename, PackagePath))
			{
				continue;
			}

			// Do not create directory placeholders in the ExternalActors of ExternalObjects folders
			if (PackagePath.Contains(FPackagePath::GetExternalActorsFolderName()) || PackagePath.Contains(FPackagePath::GetExternalObjectsFolderName()))
			{
				continue;
			}

			// Do not create directory placeholders in any folders excluded by the project settings
			const UDirectoryPlaceholderSettings* const Settings = GetDefault<UDirectoryPlaceholderSettings>();
			const bool bExclude = Algo::AnyOf(Settings->ExcludePaths, [&PackagePath](const FString& ExcludePath)
				{
					return FPaths::IsUnderDirectory(PackagePath, ExcludePath);
				});

			if (bExclude)
			{
				continue;
			}

			// Test if there are any assets already in the path of the new directory. If there are, we do not create a placeholder.
			// Note: This could occur if the directory was just renamed, copied, or moved.
			FAssetRegistryModule& AssetRegistryModule = FModuleManager::Get().GetModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry"));

			TArray<FAssetData> AssetDataList;
			constexpr bool bRecursive = true;
			if (AssetRegistryModule.Get().HasAssets(*PackagePath, bRecursive))
			{
				continue;
			}

			// Create a new UDirectoryPlaceholder asset in the directory that was just added
			const FString AssetName = FString(TEXT("UE_Placeholder"));

			IAssetTools& AssetTools = FModuleManager::GetModuleChecked<FAssetToolsModule>("AssetTools").Get();
			const UObject* const PlaceholderAsset = AssetTools.CreateAsset(AssetName, PackagePath, UDirectoryPlaceholder::StaticClass(), NewObject<UDirectoryPlaceholderFactory>());

			// If the asset was created successfully, save it now, because it will be hidden from the user by default
			// If source control is enabled, it will also be marked for add.
			if (PlaceholderAsset)
			{
				PackagesToSave.Add(PlaceholderAsset->GetPackage());
				UE_LOGF(LogDirectoryPlaceholder, Verbose, "New Directory Placeholder was created in %ls", *PackagePath);
			}
			else
			{
				UE_LOGF(LogDirectoryPlaceholder, Warning, "Failed to create new Directory Plcaeholder in %ls", *PackagePath);
			}
		}
	}

	const bool bOnlyDirty = false;
	UEditorLoadingAndSavingUtils::SavePackages(PackagesToSave, bOnlyDirty);
}

void FDirectoryPlaceholderModule::OnDeleteFolders(const TArray<FContentBrowserItemPath>& PathsToDelete)
{
	FAssetRegistryModule& AssetRegistryModule = FModuleManager::Get().GetModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry"));

	// Add the folders being deleted to an asset registry filter to check for existing placeholder assets
	FARFilter Filter;
	Filter.bRecursivePaths = true;
	for (const FContentBrowserItemPath& Path : PathsToDelete)
	{
		Filter.PackagePaths.Add(Path.GetInternalPathName());
	}

	// Find all of the assets in the folders being deleted
	TArray<FAssetData> AssetDataList;
	AssetRegistryModule.Get().GetAssets(Filter, AssetDataList);

	// If any non-placeholder assets are found, we will not delete anything automatically
	for (const FAssetData& AssetData : AssetDataList)
	{
		if (AssetData.GetClass() != UDirectoryPlaceholder::StaticClass())
		{
			return;
		}
	}

	// If all of the assets are directory placeholders, delete them from these folders
	const bool bShowConfirmation = false;
	const int32 NumAssetsDeleted = ObjectTools::DeleteAssets(AssetDataList, bShowConfirmation);

	UE_LOGF(LogDirectoryPlaceholder, Verbose, "Deleted %d Directory Placeholders", NumAssetsDeleted);
}

IMPLEMENT_MODULE(FDirectoryPlaceholderModule, DirectoryPlaceholder);

#undef LOCTEXT_NAMESPACE
