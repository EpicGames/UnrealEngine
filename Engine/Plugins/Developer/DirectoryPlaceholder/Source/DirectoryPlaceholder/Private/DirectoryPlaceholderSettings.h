// Copyright Epic Games, Inc. All Rights Reserved.
#pragma  once

#include "Engine/DeveloperSettings.h"

#include "DirectoryPlaceholderSettings.generated.h"


/**
 * Directory Placeholder Settings
 */
UCLASS(config = EditorPerProjectUserSettings)
class UDirectoryPlaceholderSettings : public UDeveloperSettings
{
	GENERATED_BODY()

public:
	UDirectoryPlaceholderSettings() = default;

	/** If enabled, Directory Placeholder assets will be automatically created in new folders */
	UPROPERTY(Config, EditAnywhere, Category = "Default")
	bool bAutomaticallyCreatePlaceholders = true;

	/** If enabled, Directory Placeholder assets will be automatically created in new folders that are created within the Project Content */
	UPROPERTY(Config, EditAnywhere, Category = "Default", meta=(EditCondition = bAutomaticallyCreatePlaceholders))
	bool bAutomaticallyCreatePlaceholdersInProjectContent = true;

	/** If enabled, Directory Placeholder assets will be automatically created in new folders that are created within the Project Plugins */
	UPROPERTY(Config, EditAnywhere, Category = "Default", meta = (EditCondition = bAutomaticallyCreatePlaceholders))
	bool bAutomaticallyCreatePlaceholdersInProjectPlugins = true;

	/** If enabled, Directory Placeholder assets will be automatically created in new folders that are created within all Additional Plugins */
	UPROPERTY(Config, EditAnywhere, Category = "Default", meta = (EditCondition = bAutomaticallyCreatePlaceholders))
	bool bAutomaticallyCreatePlaceholdersInAdditionalPlugins = true;

	/** 
	 * Directory Placeholder assets will not be automatically created in new folders under any of the paths in this list.
	 * Use content browser paths, such as "/Game/MyFolder/" or "/MyPlugin/MyFolder/".
	 */
	UPROPERTY(Config, EditAnywhere, Category = "Default", meta = (EditCondition = bAutomaticallyCreatePlaceholders))
	TArray<FString> ExcludePaths;
};
