// Copyright Epic Games, Inc. All Rights Reserved.

#include "DirectoryPlaceholderUtils.h"

#include "AssetRegistry/AssetRegistryModule.h"
#include "DirectoryPlaceholder.h"
#include "Misc/ScopedSlowTask.h"
#include "Modules/ModuleManager.h"
#include "ObjectTools.h"

#define LOCTEXT_NAMESPACE "DirectoryPlaceholderLibrary"

namespace UE::DirectoryPlaceholder::Private
{
	/** Recursively delete all unnecessary placeholder assets in this folder (and sub-folders) */
	bool CleanupPlaceholdersInternal(FName Path)
	{
		FAssetRegistryModule& AssetRegistryModule = FModuleManager::Get().GetModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry"));

		// Find all of the subfolders in this folder (non-recursively)
		TArray<FName> SubPaths;
		constexpr bool bRecursive = false;
		AssetRegistryModule.Get().GetSubPaths(Path, SubPaths, bRecursive);

		// Recursively check each of the subpaths for non-placeholder assets
		bool bSubPathsHaveAssets = false;
		FScopedSlowTask Progress(SubPaths.Num(), LOCTEXT("SlowTaskStartText", "Removing unnecessary placeholders..."));
		for (FName SubPath : SubPaths)
		{
			Progress.EnterProgressFrame(1.0f, FText::Format(LOCTEXT("SlowTaskSubText", "{0}"), FText::FromName(SubPath)));
			bSubPathsHaveAssets |= CleanupPlaceholdersInternal(SubPath);
		}

		// Get all of the assets in the current folder (non-recursive)
		TArray<FAssetData> AssetDataList;
		AssetRegistryModule.Get().GetAssetsByPath(Path, AssetDataList, bRecursive);

		// Test if there are any assets in the current path that are NOT placeholders
		const bool bHasAssets = AssetDataList.ContainsByPredicate([](const FAssetData& AssetData) { return AssetData.GetClass() != UDirectoryPlaceholder::StaticClass(); });

		// If there is at least one non-placeholder asset in this folder, or one of its subfolders, then we can safely delete the placeholder(s) in this folder
		if (bHasAssets || bSubPathsHaveAssets)
		{
			TArray<FAssetData> PlacesholdersToDelete = AssetDataList.FilterByPredicate([](const FAssetData& AssetData) {return AssetData.GetClass() == UDirectoryPlaceholder::StaticClass(); });

			if (PlacesholdersToDelete.Num() > 0)
			{
				const bool bShowConfirmation = false;
				const int32 NumAssetsDeleted = ObjectTools::DeleteAssets(PlacesholdersToDelete, bShowConfirmation);
			}
		}

		return bHasAssets || bSubPathsHaveAssets;
	}
}

void UDirectoryPlaceholderLibrary::CleanupPlaceholdersInPath(const FString& Path)
{
	TArray<FString> Paths = { Path };
	CleanupPlaceholdersInPaths(Paths);
}

void UDirectoryPlaceholderLibrary::CleanupPlaceholdersInPaths(const TArray<FString>& Paths)
{
	FAssetRegistryModule& AssetRegistryModule = FModuleManager::Get().GetModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry"));

	// The asset registry will not recognize paths that start with "/All", so remove that prefix if present
	TArray<FString> ValidPaths;
	Algo::Transform(Paths, ValidPaths, [](FString Path) 
		{ 
			Path.RemoveFromStart(TEXT("/All")); 
			return Path; 
		});

	// The only place we automatically create directory placeholders is under project content ("/Game").
	// These are also the only paths that we want to automatically delete placeholders from.
	ValidPaths = ValidPaths.FilterByPredicate([&AssetRegistryModule](FString Path)
		{ 
			return AssetRegistryModule.Get().PathExists(Path) && (Path.Equals(TEXT("/Game")) || Path.StartsWith(TEXT("/Game/")));
		});

	FScopedSlowTask Progress(ValidPaths.Num(), LOCTEXT("SlowTaskStartText", "Removing unnecessary placeholders..."));
	Progress.MakeDialog();
	for (const FString& Path : ValidPaths)
	{
		Progress.EnterProgressFrame();
		UE::DirectoryPlaceholder::Private::CleanupPlaceholdersInternal(*Path);
	}
}

void UDirectoryPlaceholderLibrary::DeletePlaceholdersInPath(const FString& Path)
{
	FAssetRegistryModule& AssetRegistryModule = FModuleManager::Get().GetModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry"));

	// Get all of the assets in the current folder
	TArray<FAssetData> AssetDataList;
	constexpr bool bRecursive = true;
	AssetRegistryModule.Get().GetAssetsByPath(*Path, AssetDataList, bRecursive);

	TArray<FAssetData> PlacesholdersToDelete = AssetDataList.FilterByPredicate([](const FAssetData& AssetData) {return AssetData.GetClass() == UDirectoryPlaceholder::StaticClass(); });

	if (PlacesholdersToDelete.Num() > 0)
	{
		const bool bShowConfirmation = false;
		const int32 NumAssetsDeleted = ObjectTools::DeleteAssets(PlacesholdersToDelete, bShowConfirmation);
	}
}

#undef LOCTEXT_NAMESPACE
