// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ContentBrowserFrontEndFilterExtension.h"

#include "DirectoryPlaceholder.generated.h"

class FFrontendFilter;
class FFrontendFilterCategory;

/** 
 * Extremely lightweight object that can be used as a placeholder in an otherwise empty directory.
 * The presence of a placeholder object allows that directory to be submitted to source control.
 */
UCLASS()
class DIRECTORYPLACEHOLDER_API UDirectoryPlaceholder : public UObject
{
	GENERATED_BODY()

public:
	UDirectoryPlaceholder() = default;
};

/** Content Browser filter used to show/hide directory placeholder assets */
UCLASS()
class UDirectoryPlaceholderSearchFilter : public UContentBrowserFrontEndFilterExtension
{
	GENERATED_BODY()

protected:
	// Begin UContentBrowserFrontEndFilterExtension Interface
	virtual void AddFrontEndFilterExtensions(TSharedPtr<FFrontendFilterCategory> DefaultCategory, TArray<TSharedRef<FFrontendFilter>>& InOutFilterList) const override;
	// End UContentBrowserFrontEndFilterExtension Interface
};
