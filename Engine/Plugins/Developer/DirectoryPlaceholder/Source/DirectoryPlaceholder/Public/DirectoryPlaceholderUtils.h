// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Kismet/BlueprintFunctionLibrary.h"

#include "DirectoryPlaceholderUtils.generated.h"

/**
 * Library functions for operations on directory placeholder assets
 */
UCLASS()
class DIRECTORYPLACEHOLDER_API UDirectoryPlaceholderLibrary : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()

public:
	/** Delete all unnecessary placeholder assets in this folder (and sub-folders) */
	UFUNCTION(BlueprintCallable, Category = "DirectoryPlaceholder")
	static void CleanupPlaceholdersInPath(const FString& Path);

	/** Delete all unnecessary placeholder assets in these folders (and sub-folders) */
	UFUNCTION(BlueprintCallable, Category = "DirectoryPlaceholder")
	static void CleanupPlaceholdersInPaths(const TArray<FString>& Paths);
	
	/** Delete all placeholder assets in this folder (and sub-folders) */
	UFUNCTION(BlueprintCallable, Category = "DirectoryPlaceholder")
	static void DeletePlaceholdersInPath(const FString& Path);
};
